{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Blank Example Plugin",
	"Description" : "An example of a minimal plugin.  This can be used as a starting point when creating your own plugin.",
	"Category" : "Examples",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "BlankPlugin",
			"Type" : "Runtime",
			"LoadingPhase" : "Default"
		}
	]
}